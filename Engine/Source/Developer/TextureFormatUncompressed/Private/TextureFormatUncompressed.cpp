// Copyright Epic Games, Inc. All Rights Reserved.

#include "Containers/SharedString.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "Interfaces/ITextureFormat.h"
#include "Interfaces/ITextureFormatModule.h"
#include "TextureCompressorModule.h"
#include "PixelFormat.h"
#include "ImageCore.h"
#include "TextureBuildFunction.h"
#include "DerivedDataBuildFunctionFactory.h"

DEFINE_LOG_CATEGORY_STATIC(LogTextureFormatUncompressed, Log, All);

class FUncompressedTextureBuildFunction final : public FTextureBuildFunction
{
	const UE::FUtf8SharedString& GetName() const final
	{
		static const UE::FUtf8SharedString Name(UTF8TEXTVIEW("UncompressedTexture"));
		return Name;
	}

	void GetVersion(UE::DerivedData::FBuildVersionBuilder& Builder, ITextureFormat*& OutTextureFormatVersioning) const final
	{
		static FGuid Version(TEXT("c04fe27a-53f6-402e-85b3-648ac6b1ad87"));
		Builder << Version;
		OutTextureFormatVersioning = FModuleManager::GetModuleChecked<ITextureFormatModule>(TEXT("TextureFormatUncompressed")).GetTextureFormat();
	}
};

/**
 * Macro trickery for supported format names.
 */
#define ENUM_SUPPORTED_FORMATS(op) \
	op(BGRA8) \
	op(G8) \
	op(G16) \
	op(VU8) \
	op(RGBA16F) \
	op(RGBA32F) \
	op(XGXR8) \
	op(RGBA8) \
	op(POTERROR) \
	op(R16F) \
	op(R32F) \
	op(R5G6B5) \
	op(A1RGB555) \
	op(RGB555A1)
#define DECL_FORMAT_NAME(FormatName) static FName GTextureFormatName##FormatName = FName(TEXT(#FormatName));
ENUM_SUPPORTED_FORMATS(DECL_FORMAT_NAME);
#undef DECL_FORMAT_NAME

#define DECL_FORMAT_NAME_ENTRY(FormatName) GTextureFormatName##FormatName ,
static FName GSupportedTextureFormatNames[] =
{
	ENUM_SUPPORTED_FORMATS(DECL_FORMAT_NAME_ENTRY)
};
#undef DECL_FORMAT_NAME_ENTRY

#undef ENUM_SUPPORTED_FORMATS

/**
 * Uncompressed texture format handler.
 */
class FTextureFormatUncompressed : public ITextureFormat
{
	virtual bool AllowParallelBuild() const override
	{
		return true;
	}

	virtual FName GetEncoderName(FName Format) const override
	{
		static const FName UncomName("Uncompressed");
		return UncomName;
	}

	virtual uint16 GetVersion(
		FName Format,
		const FTextureBuildSettings* BuildSettings
	) const override
	{
		return 0;
	}
	
	virtual FString GetDerivedDataKeyString(const FTextureBuildSettings& InBuildSettings, int32 InMipCount, const FIntVector3& InMip0Dimensions) const override
	{
		if (InBuildSettings.TextureFormatName == GTextureFormatNameRGBA16F)
		{
			return TEXT("RGBA16F");
		}
		else if (InBuildSettings.TextureFormatName == GTextureFormatNameRGBA32F)
		{
			return TEXT("RGBA32F");
		}
		else if (InBuildSettings.TextureFormatName == GTextureFormatNameR16F)
		{
			return TEXT("R16F");
		}
		else if (InBuildSettings.TextureFormatName == GTextureFormatNameR32F)
		{
			return TEXT("R32F");
		}
		else
		{
			// default implementation of GetDerivedDataKeyString returns empty string
			// match that so we don't change the DDC key

			return TEXT("");
		}
	}

	virtual void GetSupportedFormats(TArray<FName>& OutFormats) const override
	{
		OutFormats.Append(GSupportedTextureFormatNames, UE_ARRAY_COUNT(GSupportedTextureFormatNames) ); 
	}
	
	virtual EPixelFormat GetEncodedPixelFormat(const FTextureBuildSettings& BuildSettings, bool bImageHasAlphaChannel) const override
	{
		if (BuildSettings.TextureFormatName == GTextureFormatNameG8)
		{
			return PF_G8;
		}
		else if (BuildSettings.TextureFormatName == GTextureFormatNameG16)
		{
			return PF_G16;
		}
		else if (BuildSettings.TextureFormatName == GTextureFormatNameVU8)
		{
			return PF_V8U8;
		}
		else if (BuildSettings.TextureFormatName == GTextureFormatNameBGRA8 || BuildSettings.TextureFormatName == GTextureFormatNameRGBA8 ||
				 BuildSettings.TextureFormatName == GTextureFormatNameXGXR8)
		{
			return PF_B8G8R8A8;
		}
		else if (BuildSettings.TextureFormatName == GTextureFormatNameRGBA16F)
		{
			return PF_FloatRGBA;
		}
		else if (BuildSettings.TextureFormatName == GTextureFormatNameRGBA32F)
		{
			return PF_A32B32G32R32F;
		}
		else if (BuildSettings.TextureFormatName == GTextureFormatNameR16F)
		{
			return PF_R16F;
		}
		else if (BuildSettings.TextureFormatName == GTextureFormatNameR32F)
		{
			return PF_R32_FLOAT;
		}
		else if (BuildSettings.TextureFormatName == GTextureFormatNamePOTERROR)
		{
			return PF_B8G8R8A8;
		}
		else if (BuildSettings.TextureFormatName == GTextureFormatNameR5G6B5)
		{
			return PF_R5G6B5_UNORM;
		}
		else if (BuildSettings.TextureFormatName == GTextureFormatNameA1RGB555 || BuildSettings.TextureFormatName == GTextureFormatNameRGB555A1)
		{
			return PF_B5G5R5A1_UNORM;
		}

		UE_LOGF(LogTextureFormatUncompressed, Fatal, "Unhandled texture format '%ls' given to FTextureFormatUncompressed::GetEncodedPixelFormat()", *BuildSettings.TextureFormatName.ToString());
		return PF_Unknown;
	}
	
	virtual bool CanAcceptNonF32Source(FName Format) const override
	{
		return true;
	}

	// InImage arg is not actually const
	static bool DoCompressImageSimple(const FImage& InImage, ERawImageFormat::Type DestFormat, EGammaSpace DestGammaSpace, FCompressedImage2D& OutCompressedImage)
	{
		FImage Image;

		if ( InImage.Format == DestFormat && InImage.GetGammaSpace() == DestGammaSpace )
		{
			// source image is already in desired format, just move it
			// source image can be mutated by compressor
			Image.Swap(const_cast<FImage &>(InImage));
			
			// keep InImage info unchanged, only free the RawData bits
			(FImageInfo &)(const_cast<FImage &>(InImage)) = (FImageInfo &)Image;

			// note that the source bits are not freed; they're still held allocated in the OutCompressedImage
			//	currently freeing the source is not allowed because it may be being hashed on a thread
		}
		else
		{
			InImage.CopyTo(Image, DestFormat, DestGammaSpace);
		}

		OutCompressedImage.RawData = MoveTemp(Image.RawData);

		return true;
	}
	
	// InImage arg is not actually const
	virtual bool CompressImage(
		const FImage& InImage,
		const FTextureBuildSettings& BuildSettings,
		const FIntVector3& InMip0Dimensions,
		int32 InMip0NumSlicesNoDepth,
		int32 InMipIndex,
		int32 InMipCount,
		FStringView DebugTexturePathName,
		bool bImageHasAlphaChannel,
		FCompressedImage2D& OutCompressedImage
		) const override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(TFUncompressed.CompressImage);

		// InImage can be any format because CanAcceptNonF32Source

		OutCompressedImage.PixelFormat = GetEncodedPixelFormat(BuildSettings, bImageHasAlphaChannel);
		OutCompressedImage.SizeX = InImage.SizeX;
		OutCompressedImage.SizeY = InImage.SizeY;
		OutCompressedImage.NumSlicesWithDepth = InImage.NumSlices;

		if (BuildSettings.TextureFormatName == GTextureFormatNameG8)
		{
			return DoCompressImageSimple(InImage, ERawImageFormat::G8, BuildSettings.GetDestGammaSpace(), OutCompressedImage);
		}
		else if (BuildSettings.TextureFormatName == GTextureFormatNameG16)
		{
			return DoCompressImageSimple(InImage, ERawImageFormat::G16, EGammaSpace::Linear, OutCompressedImage);
		}
		else if (BuildSettings.TextureFormatName == GTextureFormatNameVU8)
		{
			FImage Image;
			InImage.CopyTo(Image, ERawImageFormat::BGRA8, BuildSettings.GetDestGammaSpace());

			int64 NumTexels = Image.GetNumPixels();
			OutCompressedImage.RawData.SetNumUninitialized(NumTexels * 2);
			const FColor* FirstColor = (&Image.AsBGRA8()[0]);
			const FColor* LastColor = FirstColor + NumTexels;
			int8* Dest = (int8*)OutCompressedImage.RawData.GetData();

			for (const FColor* Color = FirstColor; Color < LastColor; ++Color)
			{
				*Dest++ = (int32)Color->R - 128;
				*Dest++ = (int32)Color->G - 128;
			}

			return true;
		}
		else if (BuildSettings.TextureFormatName == GTextureFormatNameBGRA8)
		{
			return DoCompressImageSimple(InImage, ERawImageFormat::BGRA8, BuildSettings.GetDestGammaSpace(), OutCompressedImage);
		}
		else if (BuildSettings.TextureFormatName == GTextureFormatNameRGBA8)
		{
			FImage Image;
			InImage.CopyTo(Image, ERawImageFormat::BGRA8, BuildSettings.GetDestGammaSpace());
			
			FImageCore::TransposeImageRGBABGRA(Image);

			OutCompressedImage.RawData = MoveTemp(Image.RawData);

			return true;
		}
		else if (BuildSettings.TextureFormatName == GTextureFormatNameXGXR8)
		{
			FImage Image;
			InImage.CopyTo(Image, ERawImageFormat::BGRA8, BuildSettings.GetDestGammaSpace());

			// swizzle each texel
			int64 NumTexels = Image.GetNumPixels();
			OutCompressedImage.RawData.SetNumUninitialized(NumTexels * 4);
			const FColor* FirstColor = (&Image.AsBGRA8()[0]);
			const FColor* LastColor = FirstColor + NumTexels;
			uint8* Dest = OutCompressedImage.RawData.GetData();

			for (const FColor* Color = FirstColor; Color < LastColor; ++Color)
			{
				// XGXR8
				*Dest++ = Color->B;
				*Dest++ = Color->G;
				*Dest++ = Color->A;
				*Dest++ = Color->R;
			}

			return true;
		}
		else if (BuildSettings.TextureFormatName == GTextureFormatNameRGBA16F)
		{
			return DoCompressImageSimple(InImage, ERawImageFormat::RGBA16F, EGammaSpace::Linear, OutCompressedImage);
		}
		else if (BuildSettings.TextureFormatName == GTextureFormatNameRGBA32F)
		{
			return DoCompressImageSimple(InImage, ERawImageFormat::RGBA32F, EGammaSpace::Linear, OutCompressedImage);
		}
		else if (BuildSettings.TextureFormatName == GTextureFormatNameR16F)
		{
			return DoCompressImageSimple(InImage, ERawImageFormat::R16F, EGammaSpace::Linear, OutCompressedImage);
		}
		else if (BuildSettings.TextureFormatName == GTextureFormatNameR32F)
		{
			return DoCompressImageSimple(InImage, ERawImageFormat::R32F, EGammaSpace::Linear, OutCompressedImage);
		}
		else if (BuildSettings.TextureFormatName == GTextureFormatNamePOTERROR)
		{
			// @todo : this looks like temp debug code. please remove.

			// load the error image data we will just repeat into the texture
			TArray64<uint8> ErrorData;
			FFileHelper::LoadFileToArray(ErrorData, *(FPaths::EngineDir() / TEXT("Content/MobileResources/PowerOfTwoError64x64.raw")));
			check(ErrorData.Num() == 16384);

			// allocate output memory
			check(InImage.NumSlices == 1);
			uint64 NumTexels = (uint64)InImage.SizeX * InImage.SizeY;
			OutCompressedImage.RawData.Empty(NumTexels * 4);
			OutCompressedImage.RawData.AddUninitialized(NumTexels * 4);

			// write out texels
			uint8* Src = ErrorData.GetData();
			uint8* Dest = OutCompressedImage.RawData.GetData();
			for (int32 Y = 0; Y < InImage.SizeY; Y++)
			{
				for (int32 X = 0; X < InImage.SizeX * 4; X++)
				{
					int32 SrcX = X & (64 * 4 - 1);
					int32 SrcY = Y & 63;
					Dest[(uint64)Y * InImage.SizeX * 4 + X] = Src[(uint64)SrcY * 64 * 4 + SrcX];
				}
			}
			
			return true;
		}
		else if (BuildSettings.TextureFormatName == GTextureFormatNameR5G6B5 || BuildSettings.TextureFormatName == GTextureFormatNameRGB555A1 || BuildSettings.TextureFormatName == GTextureFormatNameA1RGB555)
		{
			FImage Image;
			InImage.CopyTo(Image, ERawImageFormat::BGRA8, BuildSettings.GetDestGammaSpace());

			// swizzle each texel
			int64 NumTexels = Image.GetNumPixels();
			OutCompressedImage.RawData.Empty(NumTexels * 2);
			OutCompressedImage.RawData.AddUninitialized(NumTexels * 2);
			const FColor* FirstColor = (&Image.AsBGRA8()[0]);
			const FColor* LastColor = FirstColor + NumTexels;
			uint16* Dest = (uint16*)OutCompressedImage.RawData.GetData();

			// @todo Oodle : this is not a correct 565 color quantizer.  See rrColor565Bits_Quantize.
			//	 not a big deal because this is rarely used these days, but beware do not copy-paste and spread this mistake
			//	(if you fix this, must bump DDC key)

			if(BuildSettings.TextureFormatName == GTextureFormatNameR5G6B5)
			{
				for (const FColor* Color = FirstColor; Color < LastColor; ++Color)
				{
					uint16 BGR565 = (uint16(Color->R >> 3) << 11) | (uint16(Color->G >> 2) << 5) | uint16(Color->B >> 3);
					*Dest++ = BGR565;
				}
			}
			else if(BuildSettings.TextureFormatName == GTextureFormatNameA1RGB555)
			{
				for (const FColor* Color = FirstColor; Color < LastColor; ++Color)
				{
					//most rhi supports alpha on the highest bit
					uint16 BGR555A1 = (uint16(Color->A >> 7) << 15) | (uint16(Color->R >> 3) << 10) | (uint16(Color->G >> 3) << 5) | uint16(Color->B >> 3);
					*Dest++ = BGR555A1;
				}
			}
			else
			{
				for (const FColor* Color = FirstColor; Color < LastColor; ++Color)
				{
					//OpenGL GL_RGB5_A1 only supports alpha on the lowest bit
					uint16 BGR555A1 = (uint16(Color->R >> 3) << 11) | (uint16(Color->G >> 3) << 6) | (uint16(Color->B >> 3) << 1) | uint16(Color->A >> 7);
					*Dest++ = BGR555A1;
				}
			}
			return true;
		}
		else
		{
			UE_LOGF(LogTextureFormatUncompressed, Warning,
				"Cannot convert uncompressed image to format '%ls'.",
				*BuildSettings.TextureFormatName.ToString()
				);

			return false;
		}
	}
};

/**
 * Module for uncompressed texture formats.
 */
static ITextureFormat* Singleton = NULL;

class FTextureFormatUncompressedModule : public ITextureFormatModule
{
public:
	virtual ~FTextureFormatUncompressedModule()
	{
		delete Singleton;
		Singleton = NULL;
	}
	
	virtual bool CanCallGetTextureFormats() override { return false; }

	virtual ITextureFormat* GetTextureFormat()
	{
		if (!Singleton)
		{
			Singleton = new FTextureFormatUncompressed();
		}
		return Singleton;
	}

	static inline UE::DerivedData::TBuildFunctionFactory<FUncompressedTextureBuildFunction> BuildFunctionFactory;
};

IMPLEMENT_MODULE(FTextureFormatUncompressedModule, TextureFormatUncompressed);

