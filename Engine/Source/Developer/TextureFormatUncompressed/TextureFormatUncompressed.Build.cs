// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TextureFormatUncompressed : ModuleRules
{
	public TextureFormatUncompressed(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateIncludePathModuleNames.AddRange(new string[]
		{
			"DerivedDataCache",
			"TextureCompressor",
			"TextureFormat",
		});

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"Core",
			"ImageCore",
			"TextureBuild",
		});
	}
}
